{
	"FileVersion" : 3,
	
	"FriendlyName" : "Chunk Downloader",
	"Version" : 1,
	"VersionName" : "1.0",
	"Description" : "Implements a streaming install client",
	"Category" : "Online Platform",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : true,

    "Modules": [
        {
            "Name": "ChunkDownloader",
            "Type": "Runtime",
            "LoadingPhase" : "PostConfigInit"
        }
    ]
}
