// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChunkDownloader : ModuleRules
{
	public ChunkDownloader(ReadOnlyTargetRules Target) : base(Target)
	{
        PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
                "ApplicationCore",
                "CoreUObject",
                "HTTP",
			}
		);
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
			}
		);
	}
}
