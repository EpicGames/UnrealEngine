// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChunkDownloader.h"

#include "ChunkDownloaderLog.h"
#include "Async/AsyncWork.h"
#include "HAL/FileManager.h"
#include "HttpModule.h"
#include "Misc/CoreDelegates.h"
#include "Interfaces/IHttpRequest.h"
#include "Misc/SecureHash.h"
#include "Interfaces/IHttpResponse.h"
#include "UObject/UObjectGlobals.h"
#include "Misc/ConfigCacheIni.h"
#include "Download.h"
#include "Modules/ModuleManager.h"

#if PLATFORM_ANDROID || PLATFORM_IOS
#include "HAL/PlatformApplicationMisc.h"
#endif

#define LOCTEXT_NAMESPACE "ChunkDownloader"

static const FString EMBEDDED_MANIFEST = TEXT("EmbeddedManifest.txt");
static const FString LOCAL_MANIFEST = TEXT("LocalManifest.txt");
static const FString CACHED_BUILD_MANIFEST = TEXT("CachedBuildManifest.txt");
static const FString BUILD_ID_KEY = TEXT("BUILD_ID");

//static 
bool FChunkDownloader::CheckFileSha1Hash(const FString& FullPathOnDisk, const FString& Sha1HashStr)
{
	IFileHandle* FilePtr = IPlatformFile::GetPlatformPhysical().OpenRead(*FullPathOnDisk);
	if (FilePtr == nullptr)
	{
		UE_LOGF(LogChunkDownloader, Error, "Unable to open %ls for hash verify.", *FullPathOnDisk);
		return false;
	}

	// create a SHA1 reader
	FSHA1 HashContext;

	// read in 64K chunks to prevent raising the memory high water mark too much
	{
		static const int64 FILE_BUFFER_SIZE = 64 * 1024;
		uint8 Buffer[FILE_BUFFER_SIZE];
		int64 FileSize = FilePtr->Size();
		for (int64 Pointer = 0; Pointer<FileSize;)
		{
			// how many bytes to read in this iteration
			int64 SizeToRead = FileSize - Pointer;
			if (SizeToRead > FILE_BUFFER_SIZE)
			{
				SizeToRead = FILE_BUFFER_SIZE;
			}

			// read dem bytes
			if (!FilePtr->Read(Buffer, SizeToRead))
			{
				UE_LOGF(LogChunkDownloader, Error, "Read error while validating '%ls' at offset %lld.", *FullPathOnDisk, Pointer);

				// don't forget to close
				delete FilePtr;
				return false;
			}
			Pointer += SizeToRead;

			// update the hash
			HashContext.Update(Buffer, SizeToRead);
		}

		// done with the file
		delete FilePtr;
	}

	// close up shop
	HashContext.Final();
	uint8 FinalHash[FSHA1::DigestSize];
	HashContext.GetHash(FinalHash);

	// build the hash string we just computed
	FString LocalHashStr = TEXT("SHA1:");
	for (int Idx = 0; Idx < 20; Idx++)
	{
		LocalHashStr += FString::Printf(TEXT("%02X"), FinalHash[Idx]);
	}
	return Sha1HashStr == LocalHashStr;
}

static bool WriteStringAsUtf8TextFile(const FString& FileText, const FString& FilePath)
{
	// convert to UTF8
	FTCHARToUTF8 PakFileUtf8(*FileText);

	// open the file for writing
	bool bSuccess = false;
	IFileHandle* ManifestFile = IPlatformFile::GetPlatformPhysical().OpenWrite(*FilePath);
	if (ManifestFile != nullptr)
	{
		// write to the file
		if (ManifestFile->Write(reinterpret_cast<const uint8*>(PakFileUtf8.Get()), PakFileUtf8.Length()))
		{
			UE_LOGF(LogChunkDownloader, Log, "Wrote to %ls", *FilePath);
			bSuccess = true;
		}
		else
		{
			UE_LOGF(LogChunkDownloader, Error, "Write error writing to %ls", *FilePath);
		}

		// close the file
		delete ManifestFile;
	}
	else
	{
		UE_LOGF(LogChunkDownloader, Error, "Unable open %ls for writing.", *FilePath);
	}
	return bSuccess;
}

////////////////////////////////////////////////////////////////////////////////////////////

class FChunkDownloader::FMultiCallback
{
public:
	FMultiCallback(const FCallback& Callback)
		: OuterCallback(Callback)
	{
		IndividualCb = [this](bool bSuccess) {
			// update stats
			--NumPending;
			if (bSuccess)
				++NumSucceeded;
			else
				++NumFailed;

			// if we're the last one, trigger the outer callback
			if (NumPending <= 0)
			{
				check(NumPending == 0);
				if (OuterCallback)
				{
					OuterCallback(NumFailed <= 0);
				}

				// done with this
				delete this;
			}
		};
	}

	inline const FCallback& AddPending()
	{ 
		++NumPending;
		return IndividualCb;
	}

	inline int GetNumPending() const { return NumPending; }

	void Abort()
	{
		check(NumPending == 0);
		delete this;
	}

private:
	~FMultiCallback() {}

	int NumPending = 0;
	int NumSucceeded = 0;
	int NumFailed = 0;
	FCallback IndividualCb;
	FCallback OuterCallback;
};

////////////////////////////////////////////////////////////////////////////////////////////

class FChunkDownloader::FPakMountWork : public FNonAbandonableTask
{
public:
	friend class FAsyncTask<FPakMountWork>;

	void DoWork()
	{
		// try to mount the pak file
		if (FCoreDelegates::MountPak.IsBound())
		{
			uint32 PakReadOrder = PakFiles.Num();
			for (const TSharedRef<FPakFile>& PakFile : PakFiles)
			{
				FString FullPathOnDisk = (PakFile->bIsEmbedded ? EmbeddedFolder : CacheFolder) / PakFile->Entry.FileName;
				IPakFile* MountedPak = FCoreDelegates::MountPak.Execute(FullPathOnDisk, PakReadOrder);

#if !UE_BUILD_SHIPPING
				if (!MountedPak)
				{
					// This can fail because of the sandbox system - which the pak system doesn't understand.
					FString SandboxedPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*FullPathOnDisk);
					MountedPak = FCoreDelegates::MountPak.Execute(SandboxedPath, PakReadOrder);
				}
#endif

				if (MountedPak)
				{
					// record that we successfully mounted this pak file
					MountedPakFiles.Add(PakFile);
					--PakReadOrder;
				}
				else
				{
					UE_LOGF(LogChunkDownloader, Error, "Unable to mount %ls from chunk %d (mount operation failed)", *FullPathOnDisk, ChunkId);
				}
			}
		}
		else
		{
			UE_LOGF(LogChunkDownloader, Error, "Unable to mount chunk %d (no FCoreDelegates::MountPak bound)", ChunkId);
		}
	}

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FPakMountWork, STATGROUP_ThreadPoolAsyncTasks);
	}

public: // inputs

	int32 ChunkId;

	// folders to save pak files into on disk
	FString CacheFolder;
	FString EmbeddedFolder;

	// mount these IN ORDER
	TArray<TSharedRef<FPakFile>> PakFiles;

	// callbacks
	TArray<FCallback> PostMountCallbacks;

public: // results

	// files which were successfully mounted
	TArray<TSharedRef<FPakFile>> MountedPakFiles;
};

////////////////////////////////////////////////////////////////////////////////////////////

FChunkDownloader::FChunkDownloader()
{
}

FChunkDownloader::~FChunkDownloader()
{
	// this will be true unless we forgot to have Finalize called.
	check(PakFiles.Num() <= 0);
}

static TArray<FPakFileEntry> ParseManifest(const FString& ManifestPath, TMap<FString,FString>* Properties = nullptr)
{
	int32 ExpectedEntries = -1;
	TArray<FPakFileEntry> Entries;
	IFileHandle* ManifestFile = IPlatformFile::GetPlatformPhysical().OpenRead(*ManifestPath);
	if (ManifestFile != nullptr)
	{
		const uint64 FileSize = ManifestFile->Size();
		if (FileSize > 0)
		{
			UE_LOGF(LogChunkDownloader, Log, "Found manifest at %ls", *ManifestPath);

			// read the whole file into a buffer (expecting UTF-8 so null terminate)
			char* FileBuffer = new char[FileSize + 8]; // little extra since we're forcing null term in places outside bounds of a field
			if (ManifestFile->Read((uint8*)FileBuffer, FileSize))
			{
				FileBuffer[FileSize] = '\0';

				// make buffers for stuff read from each line
				char NameBuffer[512] = { 0 };
				uint64 FinalFileLen = 0;
				char VersionBuffer[512] = { 0 };
				int32 ChunkId = -1;
				char RelativeUrl[2048] = { 0 };

				// line end
				uint64 LineNum = 0;
				uint64 NextLineStart = 0;
				while (NextLineStart < FileSize)
				{
					uint64 LineStart = NextLineStart;
					++LineNum;

					// find the end of the line
					uint64 LineEnd = LineStart;
					while (LineEnd < FileSize && FileBuffer[LineEnd] != '\n' && FileBuffer[LineEnd] != '\r')
					{
						++LineEnd;
					}

					// find the end of the line
					NextLineStart = LineEnd + 1;
					while (NextLineStart < FileSize && (FileBuffer[NextLineStart] == '\n' || FileBuffer[NextLineStart] == '\r'))
					{
						++NextLineStart;
					}

					// see if this is a property
					if (FileBuffer[LineStart] == '$')
					{
						// parse the line
						char* NameStart = &FileBuffer[LineStart+1];
						char* NameEnd = FCStringAnsi::Strstr(NameStart, " = ");
						if (NameEnd != nullptr)
						{
							char* ValueStart = NameEnd + 3;
							char* ValueEnd = &FileBuffer[LineEnd];
							*NameEnd = '\0';
							*ValueEnd = '\0';

							FString Name = FUTF8ToTCHAR(NameStart, NameEnd-NameStart + 1).Get();
							FString Value = FUTF8ToTCHAR(ValueStart, ValueEnd-ValueStart + 1).Get();
							if (Properties != nullptr)
							{
								Properties->Add(Name,Value);
							}

							if (Name == TEXT("NUM_ENTRIES"))
							{
								ExpectedEntries = FCString::Atoi(*Value);
							}
						}
						continue;
					}

					// parse the line
#if PLATFORM_WINDOWS || PLATFORM_MICROSOFT
					if (!ensure(sscanf_s(&FileBuffer[LineStart], "%511[^\t]\t%llu\t%511[^\t]\t%d\t%2047[^\r\n]", 
						NameBuffer, (int)sizeof(NameBuffer), 
						&FinalFileLen, 
						VersionBuffer, (int)sizeof(VersionBuffer), 
						&ChunkId, 
						RelativeUrl, (int)sizeof(RelativeUrl)
					) == 5))
#else
					if (!ensure(sscanf(&FileBuffer[LineStart], "%511[^\t]\t%llu\t%511[^\t]\t%d\t%2047[^\r\n]", 
						NameBuffer, 
						&FinalFileLen, 
						VersionBuffer, 
						&ChunkId, 
						RelativeUrl
					) == 5))
#endif
					{
						UE_LOG(LogChunkDownloader, Error, TEXT("Manifest parse error at %s:%" UINT64_FMT), *ManifestPath, LineNum);
						continue;
					}

					// add a new pak file entry
					FPakFileEntry Entry;
					Entry.FileName = UTF8_TO_TCHAR(NameBuffer);
					Entry.FileSize = FinalFileLen;
					Entry.FileVersion = UTF8_TO_TCHAR(VersionBuffer);
					if (ChunkId >= 0)
					{
						Entry.ChunkId = ChunkId;
						Entry.RelativeUrl = UTF8_TO_TCHAR(RelativeUrl);
					}
					Entries.Add(Entry);
				}

				// all done
				delete[] FileBuffer;
			}
			else
			{
				UE_LOGF(LogChunkDownloader, Error, "Read error loading manifest at %ls", *ManifestPath);
			}
		}
		else
		{
			UE_LOGF(LogChunkDownloader, Log, "Empty manifest found at %ls", *ManifestPath);
		}
		
		// close the file
		delete ManifestFile;
	}
	else
	{
		UE_LOGF(LogChunkDownloader, Log, "No manifest found at %ls", *ManifestPath);
	}

	if (ExpectedEntries >= 0 && ExpectedEntries != Entries.Num())
	{
		UE_LOGF(LogChunkDownloader, Error, "Corrupt manifest at %ls (expected %d entries, got %d)", *ManifestPath, ExpectedEntries, Entries.Num());
		Entries.Empty();
		if (Properties != nullptr)
		{
			Properties->Empty();
		}
	}

	return Entries;
}

void FChunkDownloader::Initialize(const FString& InPlatformName, int32 TargetDownloadsInFlightIn)
{
	check(PakFiles.Num() == 0); // this means we didn't call Finalize
	check(!InPlatformName.IsEmpty());
	UE_LOGF(LogChunkDownloader, Display, "Initializing with platform='%ls'", *InPlatformName);
    
    FPlatformMisc::AddAdditionalRootDirectory(FPaths::Combine(*FPaths::ProjectPersistentDownloadDir(), TEXT("pakcache")));

	// save platform name
	PlatformName = InPlatformName;

	// save target concurrency
	TargetDownloadsInFlight = TargetDownloadsInFlightIn;
	check(TargetDownloadsInFlight >= 1);

	// figure out our base dirs
	CacheFolder = FPaths::ProjectPersistentDownloadDir() / TEXT("PakCache/");
	EmbeddedFolder = FPaths::ProjectContentDir() / TEXT("EmbeddedPaks/");

	// make sure the cache folder exists
	IFileManager& FileManager = IFileManager::Get();
	if (!FileManager.MakeDirectory(*CacheFolder, true))
	{
		UE_LOGF(LogChunkDownloader, Error, "Unable to create cache folder at '%ls'", *CacheFolder);
	}

	// see what's in the embedded chunks folder
	EmbeddedPaks.Empty();
	for (const FPakFileEntry& Entry : ParseManifest(EmbeddedFolder / EMBEDDED_MANIFEST))
	{
		// just index these
		EmbeddedPaks.Add(Entry.FileName, Entry);
	}

	// enumerate the caches dir and delete anything not in the local manifest
	TArray<FString> StrayFiles;
	FileManager.FindFiles(StrayFiles, *CacheFolder, TEXT("*.pak"));

	// load the LocalManifest to see what we've got on disk
	TArray<FPakFileEntry> LocalManifest = ParseManifest(CacheFolder / LOCAL_MANIFEST);
	if (LocalManifest.Num() > 0)
	{
		// make entries in PakFileInfo for each thing in the local cache (will fill in when BuildManifest is loaded)
		for (const FPakFileEntry& Entry : LocalManifest)
		{
			// make a new file info
			TSharedRef<FPakFile> FileInfo = MakeShared<FPakFile>();

			// copy over entry fields (more will be filled in by BuildManifest)
			FileInfo->Entry = Entry;

			// see if there's a partial or cached file
			FString LocalPath = CacheFolder / Entry.FileName;
			int64 SizeOnDiskInt = FileManager.FileSize(*LocalPath);
			if (SizeOnDiskInt > 0)
			{
				FileInfo->SizeOnDisk = (uint64)SizeOnDiskInt;
				if (FileInfo->SizeOnDisk > Entry.FileSize)
				{
					// abort adding this file info (it's too big, we'll delete it)
					UE_LOGF(LogChunkDownloader, Warning, "Found '%ls' on disk with size larger than LocalManifest indicates", *LocalPath);
					bNeedsManifestSave = true;
					continue;
				}

				// see if this is a fullly cached file
				if (FileInfo->SizeOnDisk == Entry.FileSize)
				{
					// consider size match to be fully downloaded
					FileInfo->bIsCached = true;
				}

				// add the info
				PakFiles.Add(Entry.FileName, FileInfo);
			}
			else
			{
				// remove this from the local manifest and resave (may be that we crashed before the file download successfully started)
				UE_LOGF(LogChunkDownloader, Log, "'%ls' appears in LocalManifest but is not on disk (not necessarily a problem)", *LocalPath);
				bNeedsManifestSave = true;
			}

			// remove from StrayFiles
			StrayFiles.RemoveSingle(Entry.FileName);
		}
	}

	// delete any stray files that weren't in the local manifest
	for (FString Orphan : StrayFiles)
	{
		bNeedsManifestSave = true;
		FString FullPathOnDisk = CacheFolder / Orphan;
		UE_LOGF(LogChunkDownloader, Log, "Deleting orphaned file '%ls'", *FullPathOnDisk);
		if (!ensure(FileManager.Delete(*FullPathOnDisk)))
		{
			// log an error (best we can do)
			UE_LOGF(LogChunkDownloader, Error, "Unable to delete '%ls'", *FullPathOnDisk);
		}
	}

	// resave the local manifest
	SaveLocalManifest(false);
}

bool FChunkDownloader::LoadCachedBuild(const FString& DeploymentName)
{
	// try to re-populate ContentBuildId and the cached manifest
	TMap<FString, FString> CachedManifestProps;
	TArray<FPakFileEntry> CachedManifest = ParseManifest(CacheFolder / CACHED_BUILD_MANIFEST, &CachedManifestProps);
	const FString* BuildId = CachedManifestProps.Find(BUILD_ID_KEY);
	if (BuildId == nullptr || BuildId->IsEmpty())
	{
		return false;
	}

	SetContentBuildId(DeploymentName, *BuildId);
	LoadManifest(CachedManifest);
	return true;
}

void FChunkDownloader::SetContentBuildId(const FString& DeploymentName, const FString& NewContentBuildId)
{
	// save the content build id
	ContentBuildId = NewContentBuildId;
	LastDeploymentName = DeploymentName;
	UE_LOGF(LogChunkDownloader, Display, "Deployment = %ls, ContentBuildId = %ls", *DeploymentName, *ContentBuildId);

	// read CDN urls from deployment configs
	TArray<FString> CdnBaseUrls;
	FString ConfigSectionName = FString::Printf(TEXT("/Script/Plugins.ChunkDownloader %s"), *DeploymentName);
	GConfig->GetArray(*ConfigSectionName, TEXT("CdnBaseUrls"), CdnBaseUrls, GGameIni);
	if (CdnBaseUrls.Num() <= 0)
	{
		// fall back to generic config
		GConfig->GetArray(TEXT("/Script/Plugins.ChunkDownloader"), TEXT("CdnBaseUrls"), CdnBaseUrls, GGameIni);
		if (CdnBaseUrls.Num() <= 0)
		{
			UE_LOGF(LogChunkDownloader, Warning, "No CDN base URLs configured in [%ls]. Chunk downloading will only be able to use embedded cache.", *ConfigSectionName);
		}
	}

	// combine CdnBaseUrls with ContentBuildId
	BuildBaseUrls.Empty();
	for (int32 i=0,n=CdnBaseUrls.Num();i<n;++i)
	{
		const FString& BaseUrl = CdnBaseUrls[i];
		check(!BaseUrl.IsEmpty());
		FString BuildUrl = BaseUrl / ContentBuildId;
		UE_LOGF(LogChunkDownloader, Display, "ContentBaseUrl[%d] = %ls", i, *BuildUrl);
		BuildBaseUrls.Add(BuildUrl);
	}
}

void FChunkDownloader::UpdateBuild(const FString& DeploymentName, const FString& ContentBuildIdIn, const FCallback& Callback)
{
	check(!ContentBuildIdIn.IsEmpty());

	// if the build ID hasn't changed, there's no work to do
	if (ContentBuildIdIn == ContentBuildId && LastDeploymentName == DeploymentName)
	{
		ExecuteNextTick(Callback, true);
		return;
	}
	SetContentBuildId(DeploymentName, ContentBuildIdIn);

	// no overlapped UpdateBuild calls allowed, and Callback is required
	check(!UpdateBuildCallback);
	check(Callback);
	UpdateBuildCallback = Callback;

	// start the load/download process
	TryLoadBuildManifest(0);
}

void FChunkDownloader::Finalize()
{
	UE_LOGF(LogChunkDownloader, Display, "Finalizing.");

	// wait for all mounts to finish
	WaitForMounts();

	// update the mount tasks (queues up callbacks)
	ensure(UpdateMountTasks(0.0f) == false);

	// cancel all downloads
	for (const auto& It : PakFiles)
	{
		const TSharedRef<FPakFile>& File = It.Value;
		if (File->Download.IsValid())
		{
			CancelDownload(File, false);
		}
	}

	// unmount all mounted chunks (best effort)
	for (const auto& It : Chunks)
	{
		const TSharedRef<FChunk>& Chunk = It.Value;
		if (Chunk->bIsMounted)
		{
			// unmount the paks (in reverse order)
			for (int32 i=Chunk->PakFiles.Num()-1;i >= 0; --i)
			{
				const TSharedRef<FPakFile>& PakFile = Chunk->PakFiles[i];
				UnmountPakFile(PakFile);
			}

			// clear the flag
			Chunk->bIsMounted = false;
		}
	}

	// clear pak files and chunks
	PakFiles.Empty();
	Chunks.Empty();

	// cancel any pending manifest request
	if (ManifestRequest.IsValid())
	{
		ManifestRequest->CancelRequest();
		ManifestRequest.Reset();
	}

	// any loading mode is de-facto complete
	if (PostLoadCallbacks.Num() > 0)
	{
		TArray<FCallback> Callbacks = MoveTemp(PostLoadCallbacks);
		PostLoadCallbacks.Empty();
		for (const auto& Callback : Callbacks)
		{
			ExecuteNextTick(Callback, false);
		}
	}

	// update is also de-facto complete
	if (UpdateBuildCallback)
	{
		FCallback Callback = MoveTemp(UpdateBuildCallback);
		ExecuteNextTick(Callback, false);
	}

	// clear out the content build id
	ContentBuildId.Empty();
}

void FChunkDownloader::SaveLocalManifest(bool bForce)
{
	if (bForce || bNeedsManifestSave)
	{
		// build the whole file into an FString (wish we could stream it out)
		int32 NumEntries = 0;
		for (const auto& It : PakFiles)
		{
			if (!It.Value->bIsEmbedded)
			{
				if (It.Value->SizeOnDisk > 0 || It.Value->Download.IsValid())
				{
					++NumEntries;
				}
			}
		}
		FString PakFileText = FString::Printf(TEXT("$NUM_ENTRIES = %d\n"), NumEntries);
		for (const auto& It : PakFiles)
		{
			if (!It.Value->bIsEmbedded)
			{
				if (It.Value->SizeOnDisk > 0 || It.Value->Download.IsValid())
				{
					// local manifest
					const FPakFileEntry& PakFile = It.Value->Entry;
					PakFileText += FString::Printf(TEXT("%s\t%llu\t%s\t-1\t/\n"), *PakFile.FileName, PakFile.FileSize, *PakFile.FileVersion);
				}
			}
		}

		// write the file
		FString ManifestPath = CacheFolder / LOCAL_MANIFEST;
		if (WriteStringAsUtf8TextFile(PakFileText, ManifestPath))
		{
			// mark that we have saved
			bNeedsManifestSave = false;
		}
	}
}

void FChunkDownloader::WaitForMounts()
{
	bool bWaiting = false;

	for (const auto& It : Chunks)
	{
		const TSharedRef<FChunk>& Chunk = It.Value;
		if (Chunk->MountTask != nullptr)
		{
			if (!bWaiting)
			{
				UE_LOGF(LogChunkDownloader, Display, "Waiting for chunk mounts to complete...");
				bWaiting = true;
			}

			// wait for the async task to end
			Chunk->MountTask->EnsureCompletion(true);

			// complete the task on the main thread
			CompleteMountTask(*Chunk);
			check(Chunk->MountTask == nullptr);
		}
	}

	if (bWaiting)
	{
		UE_LOGF(LogChunkDownloader, Display, "...chunk mounts finished.");
	}
}

void FChunkDownloader::CancelDownload(const TSharedRef<FPakFile>& PakFile, bool bResult)
{
	if (PakFile->Download.IsValid())
	{
		// cancel the download itself
		PakFile->Download->Cancel(bResult);
		check(!PakFile->Download.IsValid());
	}
}

void FChunkDownloader::UnmountPakFile(const TSharedRef<FPakFile>& PakFile)
{
	// if it's already unmounted, don't do anything
	if (PakFile->bIsMounted)
	{
		// unmount
		if (ensure(FCoreDelegates::OnUnmountPak.IsBound()))
		{
			FString FullPathOnDisk = (PakFile->bIsEmbedded ? EmbeddedFolder : CacheFolder) / PakFile->Entry.FileName;
			if (ensure(FCoreDelegates::OnUnmountPak.Execute(FullPathOnDisk)))
			{
				// clear the mounted flag
				PakFile->bIsMounted = false;
			}
			else
			{
				UE_LOGF(LogChunkDownloader, Error, "Unable to unmount %ls", *FullPathOnDisk);
			}
		}
		else
		{
			UE_LOGF(LogChunkDownloader, Error, "Unable to unmount %ls because no OnUnmountPak is bound", *PakFile->Entry.FileName);
		}
	}
}

FChunkDownloader::EChunkStatus FChunkDownloader::GetChunkStatus(int32 ChunkId) const
{
	// do we know about this chunk at all?
	const TSharedRef<FChunk>* ChunkPtr = Chunks.Find(ChunkId);
	if (ChunkPtr == nullptr)
	{
		return EChunkStatus::Unknown;
	}
	const FChunk& Chunk = **ChunkPtr;

	// if it has no pak files, treat it the same as not found (shouldn't happen)
	if (!ensure(Chunk.PakFiles.Num() > 0))
	{
		return EChunkStatus::Unknown;
	}

	// see if it's fully mounted
	if (Chunk.bIsMounted)
	{
		return EChunkStatus::Mounted;
	}

	// count the number of paks in flight vs local
	int32 NumPaks = Chunk.PakFiles.Num(), NumCached = 0, NumDownloading = 0;
	for (const TSharedRef<FPakFile>& PakFile : Chunk.PakFiles)
	{
		if (PakFile->bIsCached)
		{
			++NumCached;
		}
		else if (PakFile->Download.IsValid())
		{
			++NumDownloading;
		}
	}

	if (NumCached >= NumPaks)
	{
		// all cached
		return EChunkStatus::Cached;
	}
	else if (NumCached + NumDownloading >= NumPaks)
	{
		// some downloads still in progress
		return EChunkStatus::Downloading;
	}
	else if (NumCached + NumDownloading > 0)
	{
		// any progress at all? (might be paused or partially preserved from manifest update)
		return EChunkStatus::Partial;
	}

	// nothing
	return EChunkStatus::Remote;
}

void FChunkDownloader::GetAllChunkIds(TArray<int32>& OutChunkIds) const
{
	Chunks.GetKeys(OutChunkIds);
}

// static
void FChunkDownloader::DumpLoadedChunks()
{
#if !WITH_EDITOR
	TSharedRef<FChunkDownloader> ChunkDownloader = FChunkDownloader::GetChecked();

	TArray<int32> ChunkIdList;
	ChunkDownloader->GetAllChunkIds(ChunkIdList);

	UE_LOGF(LogChunkDownloader, Display, "Dumping loaded chunk status\n--------------------------");
	for (int32 ChunkId : ChunkIdList)
	{
		auto ChunkStatus = ChunkDownloader->GetChunkStatus(ChunkId);
		UE_LOGF(LogChunkDownloader, Display, "Chunk #%d => %ls", ChunkId, ChunkStatusToString(ChunkStatus));
	}
#endif
}

//static 
const TCHAR* FChunkDownloader::ChunkStatusToString(EChunkStatus Status)
{
	switch (Status)
	{
	case EChunkStatus::Mounted: return TEXT("Mounted");
	case EChunkStatus::Cached: return TEXT("Cached");
	case EChunkStatus::Downloading: return TEXT("Downloading");
	case EChunkStatus::Partial: return TEXT("Partial");
	case EChunkStatus::Remote: return TEXT("Remote");
	case EChunkStatus::Unknown: return TEXT("Unknown");
	default: return TEXT("Invalid");
	}
}

int FChunkDownloader::FlushCache()
{
	IFileManager& FileManager = IFileManager::Get();

	// wait for all mounts to finish
	WaitForMounts();

	UE_LOGF(LogChunkDownloader, Display, "Flushing chunk caches at %ls", *CacheFolder);
	int FilesDeleted = 0, FilesSkipped = 0;
	for (const auto& It : Chunks)
	{
		const TSharedRef<FChunk>& Chunk = It.Value;
		check(Chunk->MountTask == nullptr); // we waited for mounts

		// cancel background downloads
		bool bDownloadPending = false;
		for (const TSharedRef<FPakFile>& PakFile : Chunk->PakFiles)
		{
			if (PakFile->Download.IsValid() && !PakFile->Download->HasCompleted())
			{
				// skip paks that are being downloaded
				bDownloadPending = true;
				break;
			}
		}

		// skip chunks that have a foreground download pending
		if (bDownloadPending)
		{
			for (const TSharedRef<FPakFile>& PakFile : Chunk->PakFiles)
			{
				if (PakFile->SizeOnDisk > 0)
				{
					// log that we skipped this one
					UE_LOGF(LogChunkDownloader, Warning, "Could not flush %ls (chunk %d) due to download in progress.", *PakFile->Entry.FileName, Chunk->ChunkId);
					++FilesSkipped;
				}
			}
		}
		else
		{
			// delete paks
			for (const TSharedRef<FPakFile>& PakFile : Chunk->PakFiles)
			{
				if (PakFile->SizeOnDisk > 0 && !PakFile->bIsEmbedded)
				{
					// log that we deleted this one
					FString FullPathOnDisk = CacheFolder / PakFile->Entry.FileName;
					if (ensure(FileManager.Delete(*FullPathOnDisk)))
					{
						UE_LOGF(LogChunkDownloader, Log, "Deleted %ls (chunk %d).", *FullPathOnDisk, Chunk->ChunkId);
						++FilesDeleted;

						// flag uncached (may have been partial)
						PakFile->bIsCached = false;
						PakFile->SizeOnDisk = 0;
						bNeedsManifestSave = true;
					}
					else
					{
						// log an error (best we can do)
						UE_LOGF(LogChunkDownloader, Error, "Unable to delete %ls", *FullPathOnDisk);
						++FilesSkipped;
					}
				}
			}
		}
	}

	// resave the manifest
	SaveLocalManifest(false);

	UE_LOGF(LogChunkDownloader, Display, "Chunk cache flush complete. %d files deleted. %d files skipped.", FilesDeleted, FilesSkipped);
	return FilesSkipped;
}

int FChunkDownloader::ValidateCache()
{
	IFileManager& FileManager = IFileManager::Get();

	// wait for all mounts to finish
	WaitForMounts();

	UE_LOGF(LogChunkDownloader, Display, "Starting inline chunk validation.");
	int ValidFiles = 0, InvalidFiles = 0, SkippedFiles = 0;
	for (const auto& It : PakFiles)
	{
		const TSharedRef<FPakFile>& PakFile = It.Value;
		if (PakFile->bIsCached && !PakFile->bIsEmbedded)
		{
			// we know how to validate certain hash versions
			bool bFileIsValid = false;
			if (PakFile->Entry.FileVersion.StartsWith(TEXT("SHA1:")))
			{
				// check the sha1 hash
				bFileIsValid = CheckFileSha1Hash(CacheFolder / PakFile->Entry.FileName, PakFile->Entry.FileVersion);
			}
			else
			{
				// we don't know how to validate this version format
				UE_LOGF(LogChunkDownloader, Warning, "Unable to validate %ls with version '%ls'.", *PakFile->Entry.FileName, *PakFile->Entry.FileVersion);
				++SkippedFiles;
				continue;
			}

			// see if it's valid or not
			if (bFileIsValid)
			{
				// log valid
				UE_LOGF(LogChunkDownloader, Log, "%ls matches hash '%ls'.", *PakFile->Entry.FileName, *PakFile->Entry.FileVersion);
				++ValidFiles;
			}
			else
			{
				// log invalid
				UE_LOGF(LogChunkDownloader, Warning, "%ls does NOT match hash '%ls'.", *PakFile->Entry.FileName, *PakFile->Entry.FileVersion);
				++InvalidFiles;

				// delete invalid files
				FString FullPathOnDisk = CacheFolder / PakFile->Entry.FileName;
				if (ensure(FileManager.Delete(*FullPathOnDisk)))
				{
					UE_LOGF(LogChunkDownloader, Log, "Deleted invalid pak %ls (chunk %d).", *FullPathOnDisk, PakFile->Entry.ChunkId);
					PakFile->bIsCached = false;
					PakFile->SizeOnDisk = 0;
					bNeedsManifestSave = true;
				}
			}
		}
	}

	// resave the manifest
	SaveLocalManifest(false);

	UE_LOGF(LogChunkDownloader, Display, "Chunk validation complete. %d valid, %d invalid, %d skipped", ValidFiles, InvalidFiles, SkippedFiles);
	return InvalidFiles;
}

void FChunkDownloader::BeginLoadingMode(const FCallback& Callback)
{
	check(Callback); // you can't start loading mode without a valid callback

	// see if we're already in loading mode
	if (PostLoadCallbacks.Num() > 0)
	{
		UE_LOGF(LogChunkDownloader, Log, "JoinLoadingMode");

		// just wait on the existing loading mode to finish
		PostLoadCallbacks.Add(Callback);
		return;
	}

	// start loading mode
	UE_LOGF(LogChunkDownloader, Log, "BeginLoadingMode");
#if PLATFORM_ANDROID || PLATFORM_IOS
	FPlatformApplicationMisc::ControlScreensaver(FPlatformApplicationMisc::Disable);
#endif

	// reset stats
	LoadingModeStats.LastError = FText();
	LoadingModeStats.BytesDownloaded = 0;
	LoadingModeStats.FilesDownloaded = 0;
	LoadingModeStats.ChunksMounted = 0;
	LoadingModeStats.LoadingStartTime = FDateTime::UtcNow();
	ComputeLoadingStats(); // recompute before binding callback in case there's nothing queued yet

	// set the callback
	PostLoadCallbacks.Add(Callback);
	LoadingCompleteLatch = 0;

	// compute again next frame (if nothing's queued by then, we'll fire the callback
	TWeakPtr<FChunkDownloader> WeakThisPtr = AsShared();
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([WeakThisPtr](float dts) {
		TSharedPtr<FChunkDownloader> SharedThis = WeakThisPtr.Pin();
		if (!SharedThis.IsValid() || SharedThis->PostLoadCallbacks.Num() <= 0)
		{
			return false; // stop ticking
		}
		return SharedThis->UpdateLoadingMode();
	}));
}

bool FChunkDownloader::UpdateLoadingMode()
{
	// recompute loading stats
	ComputeLoadingStats();

	// check for the end of loading mode
	if (LoadingModeStats.FilesDownloaded >= LoadingModeStats.TotalFilesToDownload &&
		LoadingModeStats.ChunksMounted >= LoadingModeStats.TotalChunksToMount)
	{
		// make sure loading's been done for at least 2 frames before firing the callback
		// this adds a negligible amount of time to the loading screen but gives dependent loads a chance to queue
		static const int32 NUM_CONSECUTIVE_IDLE_FRAMES_FOR_LOADING_COMPLETION = 5;
		if (++LoadingCompleteLatch >= NUM_CONSECUTIVE_IDLE_FRAMES_FOR_LOADING_COMPLETION)
		{
			// end loading mode
			UE_LOGF(LogChunkDownloader, Log, "EndLoadingMode (%d files downloaded, %d chunks mounted)", LoadingModeStats.FilesDownloaded, LoadingModeStats.ChunksMounted);
#if PLATFORM_ANDROID || PLATFORM_IOS
			FPlatformApplicationMisc::ControlScreensaver(FPlatformApplicationMisc::Enable);
#endif

			// fire any loading mode completion callbacks
			TArray<FCallback> Callbacks = MoveTemp(PostLoadCallbacks);
			if (Callbacks.Num() > 0)
			{
				PostLoadCallbacks.Empty(); // shouldn't be necessary due to MoveTemp but just in case
				for (const auto& Callback : Callbacks)
				{
					Callback(LoadingModeStats.LastError.IsEmpty());
				}
			}
			return false; // stop ticking
		}
	}
	else
	{
		// reset the latch
		LoadingCompleteLatch = 0;
	}

	return true; // keep ticking
}

void FChunkDownloader::ComputeLoadingStats()
{
	LoadingModeStats.TotalBytesToDownload = LoadingModeStats.BytesDownloaded;
	LoadingModeStats.TotalFilesToDownload = LoadingModeStats.FilesDownloaded;
	LoadingModeStats.TotalChunksToMount = LoadingModeStats.ChunksMounted;

	// loop over all chunks
	for (const auto& It : Chunks)
	{
		const TSharedRef<FChunk>& Chunk = It.Value;

		// if it's mounting, add files to mount
		if (Chunk->MountTask != nullptr)
		{
			++LoadingModeStats.TotalChunksToMount;
		}
	}

	// check downloads
	for (const TSharedRef<FPakFile>& PakFile : DownloadRequests)
	{
		++LoadingModeStats.TotalFilesToDownload;
		if (PakFile->Download.IsValid())
		{
			LoadingModeStats.TotalBytesToDownload += PakFile->Entry.FileSize - PakFile->Download->GetProgress();
		}
		else
		{
			LoadingModeStats.TotalBytesToDownload += PakFile->Entry.FileSize;
		}
	}
}

void FChunkDownloader::ExecuteNextTick(const FCallback& Callback, bool bSuccess)
{
	if (Callback)
	{
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([Callback, bSuccess](float dts) {
			Callback(bSuccess);
			return false;
		}));
	}
}

void FChunkDownloader::TryLoadBuildManifest(int TryNumber)
{
	// load the local build manifest
	TMap<FString, FString> CachedManifestProps;
	TArray<FPakFileEntry> CachedManifest = ParseManifest(CacheFolder / CACHED_BUILD_MANIFEST, &CachedManifestProps);

	// see if the BUILD_ID property matches
	if (CachedManifestProps.FindOrAdd(BUILD_ID_KEY) != ContentBuildId)
	{
		// if we have no CDN configured, we're done
		if (BuildBaseUrls.Num() <= 0)
		{
			UE_LOGF(LogChunkDownloader, Error, "Unable to download build manifest. No CDN urls configured.");
			LoadingModeStats.LastError = LOCTEXT("UnableToDownloadManifest", "Unable to download build manifest. (NoCDN)");

			// execute and clear the callback
			FCallback Callback = MoveTemp(UpdateBuildCallback);
			ExecuteNextTick(Callback, false);
			return;
		}

		// fast path the first try
		if (TryNumber <= 0)
		{
			// download it
			TryDownloadBuildManifest(TryNumber);
			return;
		}

		// compute delay before re-starting download
		float SecondsToDelay = TryNumber * 5.0f;
		if (SecondsToDelay > 60)
		{
			SecondsToDelay = 60;
		}

		// set a ticker to delay
		UE_LOGF(LogChunkDownloader, Log, "Will re-attempt manifest download in %f seconds", SecondsToDelay);
		TWeakPtr<FChunkDownloader> WeakThisPtr = AsShared();
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([WeakThisPtr, TryNumber](float Unused) {
			TSharedPtr<FChunkDownloader> SharedThis = WeakThisPtr.Pin();
			if (SharedThis.IsValid())
			{
				SharedThis->TryDownloadBuildManifest(TryNumber);
			}
			return false;
		}), SecondsToDelay);
		return;
	}

	// cached build manifest is up to date, load this one
	LoadManifest(CachedManifest);

	// execute and clear the callback
	FCallback Callback = MoveTemp(UpdateBuildCallback);
	ExecuteNextTick(Callback, true);
}

void FChunkDownloader::TryDownloadBuildManifest(int TryNumber)
{
	check(BuildBaseUrls.Num() > 0);

	// download the manifest from CDN, then load it
	FString ManifestFileName = FString::Printf(TEXT("BuildManifest-%s.txt"), *PlatformName);
	FString Url = BuildBaseUrls[TryNumber % BuildBaseUrls.Num()] / ManifestFileName;
	UE_LOGF(LogChunkDownloader, Log, "Downloading build manifest (attempt #%d) from %ls", TryNumber+1, *Url);

	// download the manifest from the root CDN
	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	check(!ManifestRequest.IsValid());
	ManifestRequest = HttpModule.Get().CreateRequest();
	ManifestRequest->SetURL(Url);
	ManifestRequest->SetVerb(TEXT("GET"));
	TWeakPtr<FChunkDownloader> WeakThisPtr = AsShared();
	FString CachedManifestFullPath = CacheFolder / CACHED_BUILD_MANIFEST;
	ManifestRequest->OnProcessRequestComplete().BindLambda([WeakThisPtr, TryNumber, CachedManifestFullPath](FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSuccess) {
		// if successful, save
		FText LastError;
		if (bSuccess && HttpResponse.IsValid())
		{
			const int32 HttpStatus = HttpResponse->GetResponseCode();
			if (EHttpResponseCodes::IsOk(HttpStatus))
			{
				// Save the manifest to a file
				if (!WriteStringAsUtf8TextFile(HttpResponse->GetContentAsString(), CachedManifestFullPath))
				{
					UE_LOGF(LogChunkDownloader, Error, "Failed to write manifest to '%ls'", *CachedManifestFullPath);
					LastError = FText::Format(LOCTEXT("FailedToWriteManifest", "[Try {0}] Failed to write manifest."), FText::AsNumber(TryNumber));
				}
			}
			else
			{
				UE_LOGF(LogChunkDownloader, Error, "HTTP %d while downloading manifest from '%ls'", HttpStatus, *HttpRequest->GetURL());
				LastError = FText::Format(LOCTEXT("ManifestHttpError_FailureCode", "[Try {0}] Manifest download failed (HTTP {1})"), FText::AsNumber(TryNumber), FText::AsNumber(HttpStatus));
			}
		}
		else
		{
			UE_LOGF(LogChunkDownloader, Error, "HTTP connection issue while downloading manifest '%ls'", *HttpRequest->GetURL());
			LastError = FText::Format(LOCTEXT("ManifestHttpError_Generic", "[Try {0}] Connection issues downloading manifest. Check your network connection..."), FText::AsNumber(TryNumber));
		}

		// try to load it
		TSharedPtr<FChunkDownloader> SharedThis = WeakThisPtr.Pin();
		if (!SharedThis.IsValid())
		{
			UE_LOGF(LogChunkDownloader, Warning, "FChunkDownloader was destroyed while downloading manifest '%ls'", *HttpRequest->GetURL());
			return;
		}
		SharedThis->ManifestRequest.Reset();
		SharedThis->LoadingModeStats.LastError = LastError; // ok with this clearing the error on success
		SharedThis->TryLoadBuildManifest(TryNumber + 1);
	});
	ManifestRequest->ProcessRequest();
}

// block waiting for any pending mounts to finish
// then collect garbage to clean up any references to chunks.
// then create entries for any new chunks
// for any chunks that change, cancel downloads and unmount invalid paks (and any after invalid paks).
// If a changed chunk was mounted, then inline mount all non-mounted paks in the new list (in order)  
// then unload any chunks that no longer exist (cancel downloads and unmount all paks)
void FChunkDownloader::LoadManifest(const TArray<FPakFileEntry>& ManifestPakFiles)
{
	UE_LOGF(LogChunkDownloader, Display, "Beginning manifest load.");

	// wait for all mounts to finish
	WaitForMounts();

	// trigger garbage collection (give any unmounts which are about to happen a good chance of success)
	CollectGarbage(RF_NoFlags);

	// group the manifest paks by chunk ID (maintain ordering)
	TMap<int32,TArray<FPakFileEntry>> Manifest;
	for (const FPakFileEntry& FileEntry : ManifestPakFiles)
	{
		check(FileEntry.ChunkId >= 0);
		Manifest.FindOrAdd(FileEntry.ChunkId).Add(FileEntry);
	}

	// copy old chunk map (we will reuse any that still exist)
	TMap<int32,TSharedRef<FChunk>> OldChunks = MoveTemp(Chunks);
	TMap<FString,TSharedRef<FPakFile>> OldPakFiles = MoveTemp(PakFiles);

	// loop over the new chunks
	int32 NumChunks = 0, NumPaks = 0;
	for (const auto& It : Manifest)
	{
		int32 ChunkId = It.Key;
		
		// keep track of new chunk and old pak files
		TSharedPtr<FChunk> Chunk;
		TArray<TSharedRef<FPakFile>> PrevPakList;

		// create or reuse the chunk
		TSharedRef<FChunk>* OldChunk = OldChunks.Find(ChunkId);
		if (OldChunk != nullptr)
		{
			// move over the old chunk
			Chunk = *OldChunk;
			check(Chunk->ChunkId == ChunkId);

			// don't clean it up later
			OldChunks.Remove(ChunkId);

			// move out OldPakFiles
			PrevPakList = MoveTemp(Chunk->PakFiles);
		}
		else
		{
			// make a brand new chunk
			Chunk = MakeShared<FChunk>();
			Chunk->ChunkId = ChunkId;
		}

		// add the chunk to the new map
		Chunks.Add(Chunk->ChunkId, Chunk.ToSharedRef());

		// find or create new pak files
		check(Chunk->PakFiles.Num() == 0);
		for (const FPakFileEntry& FileEntry : It.Value)
		{
			// see if there's an existing file for this one
			const TSharedRef<FPakFile>* ExistingFilePtr = OldPakFiles.Find(FileEntry.FileName);
			if (ExistingFilePtr != nullptr)
			{
				const TSharedRef<FPakFile>& ExistingFile = *ExistingFilePtr;
				if (ExistingFile->Entry.FileVersion == FileEntry.FileVersion)
				{
					// if version matched, size should too
					check(ExistingFile->Entry.FileSize == FileEntry.FileSize);

					// update and add to list (may populate ChunkId and RelativeUrl if we loaded from cache)
					ExistingFile->Entry = FileEntry;
					Chunk->PakFiles.Add(ExistingFile);
					PakFiles.Add(ExistingFile->Entry.FileName, ExistingFile);

					// remove from old pak files list
					OldPakFiles.Remove(FileEntry.FileName);
					continue;
				}
			}

			// create a new entry
			TSharedRef<FPakFile> NewFile = MakeShared<FPakFile>();
			NewFile->Entry = FileEntry;
			Chunk->PakFiles.Add(NewFile);
			PakFiles.Add(NewFile->Entry.FileName, NewFile);

			// see if it matches an embedded pak file
			const FPakFileEntry* CachedEntry = EmbeddedPaks.Find(FileEntry.FileName);
			if (CachedEntry != nullptr && CachedEntry->FileVersion == FileEntry.FileVersion)
			{
				NewFile->bIsEmbedded = true;
				NewFile->bIsCached = true;
				NewFile->SizeOnDisk = CachedEntry->FileSize;
			}
		}

		// log the chunk and pak file count
		UE_LOGF(LogChunkDownloader, Verbose, "Found chunk %d (%d pak files).", ChunkId, Chunk->PakFiles.Num());
		++NumChunks;
		NumPaks += Chunk->PakFiles.Num();

		// if the chunk is already mounted, we want to unmount any invalid data
		check(Chunk->MountTask == nullptr); // we already waited for mounts to finish
		if (Chunk->bIsMounted)
		{
			// see if all the existing pak files match to the new manifest (means it can stay mounted)
			// this is a common case so we're trying to be more efficient here
			int LongestCommonPrefix = 0;
			for (int i=0;i < PrevPakList.Num() && i < Chunk->PakFiles.Num();++i,++LongestCommonPrefix)
			{
				if (Chunk->PakFiles[i]->Entry.FileVersion != PrevPakList[i]->Entry.FileVersion)
				{
					break;
				}
			}

			// if they don't all match we need to remount
			if (LongestCommonPrefix != PrevPakList.Num() || LongestCommonPrefix != Chunk->PakFiles.Num())
			{
				// this chunk is no longer fully mounted
				Chunk->bIsMounted = false;

				// unmount any old paks that didn't match (reverse order)
				for (int i= PrevPakList.Num()-1;i>=0;--i)
				{
					UnmountPakFile(PrevPakList[i]);
				}

				// unmount any new paks that didn't match (may have changed position) (reverse order)
				// any new pak files unmounted will be re-mounted (in the right order) if this chunk is requested again
				for (int i=Chunk->PakFiles.Num()-1;i>=0;--i)
				{
					UnmountPakFile(Chunk->PakFiles[i]);
				}
			}
		}
	}

	// any files still left in OldPakFiles should be cancelled, unmounted, and deleted
	IFileManager& FileManager = IFileManager::Get();
	for (const auto& It : OldPakFiles)
	{
		const TSharedRef<FPakFile>& File = It.Value;
		UE_LOGF(LogChunkDownloader, Log, "Removing orphaned pak file %ls (was chunk %d).", *File->Entry.FileName, File->Entry.ChunkId);

		// cancel downloads of pak files that are no longer valid
		if (File->Download.IsValid())
		{
			// treat these cancellations as successful since the pak is no longer needed (we've successfully downloaded nothing)
			CancelDownload(File, true);
		}

		// if a chunk completely disappeared we may need to clean up its mounts this way (otherwise would have been taken care of above)
		if (File->bIsMounted)
		{
			UnmountPakFile(File);
		}

		// delete any locally cached file
		if (File->SizeOnDisk > 0 && !File->bIsEmbedded)
		{
			bNeedsManifestSave = true;
			FString FullPathOnDisk = CacheFolder / File->Entry.FileName;
			if (!ensure(FileManager.Delete(*FullPathOnDisk)))
			{
				UE_LOGF(LogChunkDownloader, Error, "Failed to delete orphaned pak %ls.", *FullPathOnDisk);
			}
		}
	}

	// resave the manifest
	SaveLocalManifest(false);

	// log end
	check(ManifestPakFiles.Num() == NumPaks);
	UE_LOGF(LogChunkDownloader, Display, "Manifest load complete. %d chunks with %d pak files.", NumChunks, NumPaks);
}

void FChunkDownloader::DownloadChunkInternal(const FChunk& Chunk, const FCallback& Callback, int32 Priority)
{
	UE_LOGF(LogChunkDownloader, Log, "Chunk %d download requested.", Chunk.ChunkId);

	// see if we need to download anything at all
	bool bNeedsDownload = false;
	for (const auto& PakFile : Chunk.PakFiles)
	{
		if (!PakFile->bIsCached)
		{
			bNeedsDownload = true;
			break;
		}
	}
	if (!bNeedsDownload)
	{
		ExecuteNextTick(Callback, true);
		return;
	}

	// make sure we have CDN configured
	if (BuildBaseUrls.Num() <= 0)
	{
		UE_LOGF(LogChunkDownloader, Error, "Unable to download Chunk %d (no CDN urls).", Chunk.ChunkId);
		ExecuteNextTick(Callback, false);
		return;
	}

	// download all pak files that aren't already cached
	FMultiCallback* MultiCallback = new FMultiCallback(Callback);
	for (const auto& PakFile : Chunk.PakFiles)
	{
		if (!PakFile->bIsCached)
		{
			DownloadPakFileInternal(PakFile, MultiCallback->AddPending(), Priority);
		}
	}
	check(MultiCallback->GetNumPending() > 0);
} //-V773

void FChunkDownloader::MountChunkInternal(FChunk& Chunk, const FCallback& Callback)
{
	check(!Chunk.bIsMounted);

	// see if there's already a mount pending
	if (Chunk.MountTask != nullptr)
	{
		// join with the existing callbacks
		if (Callback)
		{
			Chunk.MountTask->GetTask().PostMountCallbacks.Add(Callback);
		}
		return;
	}

	// see if we need to trigger any downloads
	bool bAllPaksCached = true;
	for (const auto& PakFile : Chunk.PakFiles)
	{
		if (!PakFile->bIsCached)
		{
			bAllPaksCached = false;
			break;
		}
	}

	if (bAllPaksCached)
	{
		// if all pak files are cached, mount now
		UE_LOGF(LogChunkDownloader, Log, "Chunk %d mount requested (%d pak sequence).", Chunk.ChunkId, Chunk.PakFiles.Num());

		// spin up a background task to mount the pak file
		check(Chunk.MountTask == nullptr);
		Chunk.MountTask = new FMountTask();

		// configure the task
		FPakMountWork& MountWork = Chunk.MountTask->GetTask();
		MountWork.ChunkId = Chunk.ChunkId;
		MountWork.CacheFolder = CacheFolder;
		MountWork.EmbeddedFolder = EmbeddedFolder;
		for (const TSharedRef<FPakFile>& PakFile : Chunk.PakFiles)
		{
			if (!PakFile->bIsMounted)
			{
				MountWork.PakFiles.Add(PakFile);
			}
		}
		if (Callback)
		{
			MountWork.PostMountCallbacks.Add(Callback);
		}

		// start as a background task
		Chunk.MountTask->StartBackgroundTask();

		// start a per-frame ticker until mounts are finished
		if (!MountTicker.IsValid())
		{
			MountTicker = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateSP(this, &FChunkDownloader::UpdateMountTasks));
		}
	}
	else
	{
		// queue up pak file downloads
		TWeakPtr<FChunkDownloader> WeakThisPtr = AsShared();
		int32 ChunkId = Chunk.ChunkId;
		DownloadChunkInternal(Chunk, [WeakThisPtr, ChunkId, Callback](bool bDownloadSuccess) {
			// if the download failed, we can't mount
			if (bDownloadSuccess)
			{
				TSharedPtr<FChunkDownloader> SharedThis = WeakThisPtr.Pin();
				if (SharedThis.IsValid())
				{
					// if all chunks are downloaded, do the mount again (this will pick up any changes and continue downloading if needed)
					SharedThis->MountChunk(ChunkId, Callback);
					return;
				}
			}

			// if anything went wrong, fire the callback now
			if (Callback)
			{
				Callback(false);
			}
		}, MAX_int32);
	}
}

void FChunkDownloader::DownloadPakFileInternal(const TSharedRef<FPakFile>& PakFile, const FCallback& Callback, int32 Priority)
{
	check(BuildBaseUrls.Num() > 0);

	// increase priority if it's updated
	if (Priority > PakFile->Priority)
	{
		// if the download has already started this won't really change anything
		PakFile->Priority = Priority;
	}

	// just piggyback on the existing post-download callback
	if (Callback)
	{
		PakFile->PostDownloadCallbacks.Add(Callback);
	}

	// see if the download is already started
	if (PakFile->Download.IsValid())
	{
		// nothing to do then (we already added our callback)
		return;
	}

	// add it to the downloading set
	DownloadRequests.AddUnique(PakFile);
	DownloadRequests.StableSort([](const TSharedRef<FPakFile>& A, const TSharedRef<FPakFile>& B) {
		return A->Priority < B->Priority;
	});

	// start the first N pak files in flight
	IssueDownloads();
}

void FChunkDownloader::IssueDownloads()
{
	for (int32 i = 0; i < DownloadRequests.Num() && i < TargetDownloadsInFlight; ++i)
	{
		TSharedRef<FPakFile> DownloadPakFile = DownloadRequests[i];
		if (DownloadPakFile->Download.IsValid())
		{
			// already downloading
			continue;
		}

		// log that we're starting a download
		UE_LOGF(LogChunkDownloader, Log, "Pak file %ls download requested (%ls).",
			*DownloadPakFile->Entry.FileName,
			*DownloadPakFile->Entry.RelativeUrl
		);
		bNeedsManifestSave = true;

		// make a new download (platform specific)
		DownloadPakFile->Download = MakeShared<FDownload>(AsShared(), DownloadPakFile);
		DownloadPakFile->Download->Start();
	}
}

void FChunkDownloader::CompleteMountTask(FChunk& Chunk)
{
	check(Chunk.MountTask != nullptr);
	check(Chunk.MountTask->IsDone());

	// increment chunks mounted
	++LoadingModeStats.ChunksMounted;

	// remove the mount
	FMountTask* Mount = Chunk.MountTask;
	Chunk.MountTask = nullptr;

	// get the work
	const FPakMountWork& MountWork = Mount->GetTask();

	// update bIsMounted on paks that actually succeeded
	for (const TSharedRef<FPakFile>& PakFile : MountWork.MountedPakFiles)
	{
		PakFile->bIsMounted = true;
	}

	// update bIsMounted on the chunk
	bool bAllPaksMounted = true;
	for (const TSharedRef<FPakFile>& PakFile : Chunk.PakFiles)
	{
		if (!PakFile->bIsMounted)
		{
			LoadingModeStats.LastError = FText::Format(LOCTEXT("FailedToMount", "Failed to mount {0}."), FText::FromString(PakFile->Entry.FileName));
			bAllPaksMounted = false;
			break;
		}
	}
	Chunk.bIsMounted = bAllPaksMounted;
	if (Chunk.bIsMounted)
	{
		UE_LOGF(LogChunkDownloader, Log, "Chunk %d mount succeeded.", Chunk.ChunkId);
	}
	else
	{
		UE_LOGF(LogChunkDownloader, Error, "Chunk %d mount failed.", Chunk.ChunkId);
	}

	// trigger the post-mount callbacks
	for (const FCallback& Callback : MountWork.PostMountCallbacks)
	{
		ExecuteNextTick(Callback, bAllPaksMounted);
	}

	// also trigger the multicast event
	OnChunkMounted.Broadcast(Chunk.ChunkId, bAllPaksMounted);

	// finally delete the task
	delete Mount;

	// recompute loading stats
	ComputeLoadingStats();
}

bool FChunkDownloader::UpdateMountTasks(float dts)
{
	bool bMountsPending = false;

	for (const auto& It : Chunks)
	{
		const TSharedRef<FChunk>& Chunk = It.Value;
		if (Chunk->MountTask != nullptr)
		{
			if (Chunk->MountTask->IsDone())
			{
				// complete it
				CompleteMountTask(*Chunk);
			}
			else
			{
				// mount still pending
				bMountsPending = true;
			}
		}
	}

	if (!bMountsPending)
	{
		MountTicker.Reset();
	}
	return bMountsPending; // keep ticking
}

void FChunkDownloader::DownloadChunk(int32 ChunkId, const FCallback& Callback, int32 Priority)
{
	// look up the chunk
	TSharedRef<FChunk>* ChunkPtr = Chunks.Find(ChunkId);
	if (ChunkPtr == nullptr || (*ChunkPtr)->PakFiles.Num() <= 0)
	{
		// a chunk that doesn't exist or one with no pak files are both considered "complete" for the purposes of this call
		// use GetChunkStatus to differentiate from chunks that mounted successfully
		UE_LOGF(LogChunkDownloader, Warning, "Ignoring download request for chunk %d (no mapped pak files).", ChunkId);
		ExecuteNextTick(Callback, true);
		return;
	}
	const FChunk& Chunk = **ChunkPtr;

	// if all the paks are cached, just succeed
	if (Chunk.IsCached())
	{
		ExecuteNextTick(Callback, true);
		return;
	}

	// queue the download
	DownloadChunkInternal(Chunk, Callback, Priority);

	// resave manifest if needed
	SaveLocalManifest(false);
	ComputeLoadingStats();
}

void FChunkDownloader::DownloadChunks(const TArray<int32>& ChunkIds, const FCallback& Callback, int32 Priority)
{
	// convert to chunk references
	TArray<TSharedRef<FChunk>> ChunksToDownload;
	for (int32 ChunkId : ChunkIds)
	{
		TSharedRef<FChunk>* ChunkPtr = Chunks.Find(ChunkId);
		if (ChunkPtr != nullptr)
		{
			TSharedRef<FChunk>& ChunkRef = *ChunkPtr;
			if (ChunkRef->PakFiles.Num() > 0)
			{
				if (!ChunkRef->IsCached())
				{
					ChunksToDownload.Add(ChunkRef);
				}
				continue;
			}
		}
		UE_LOGF(LogChunkDownloader, Warning, "Ignoring download request for chunk %d (no mapped pak files).", ChunkId);
	}

	// make sure there are some chunks to mount (saves a frame)
	if (ChunksToDownload.Num() <= 0)
	{
		// trivial success
		ExecuteNextTick(Callback, true);
		return;
	}

	// if there's no callback for some reason, avoid a bunch of boilerplate
	if (Callback)
	{
		// loop over chunks and issue individual callback
		FMultiCallback* MultiCallback = new FMultiCallback(Callback);
		for (const TSharedRef<FChunk>& Chunk : ChunksToDownload)
		{
			DownloadChunkInternal(*Chunk, MultiCallback->AddPending(), Priority);
		}
		check(MultiCallback->GetNumPending() > 0);
	} //-V773
	else
	{
		// no need to manage callbacks
		for (const TSharedRef<FChunk>& Chunk : ChunksToDownload)
		{
			DownloadChunkInternal(*Chunk, FCallback(), Priority);
		}
	}

	// resave manifest if needed
	SaveLocalManifest(false);
	ComputeLoadingStats();
}

void FChunkDownloader::MountChunk(int32 ChunkId, const FCallback& Callback)
{
	// look up the chunk
	TSharedRef<FChunk>* ChunkPtr = Chunks.Find(ChunkId);
	if (ChunkPtr == nullptr || (*ChunkPtr)->PakFiles.Num() <= 0)
	{
		// a chunk that doesn't exist or one with no pak files are both considered "complete" for the purposes of this call
		// use GetChunkStatus to differentiate from chunks that mounted successfully
		UE_LOGF(LogChunkDownloader, Warning, "Ignoring mount request for chunk %d (no mapped pak files).", ChunkId);
		ExecuteNextTick(Callback, true);
		return;
	}
	FChunk& Chunk = **ChunkPtr;

	// see if we're mounted already
	if (Chunk.bIsMounted)
	{
		// trivial success
		ExecuteNextTick(Callback, true);
		return;
	}

	// mount the chunk
	MountChunkInternal(Chunk, Callback);

	// resave manifest if needed
	SaveLocalManifest(false);
	ComputeLoadingStats();
}

void FChunkDownloader::MountChunks(const TArray<int32>& ChunkIds, const FCallback& Callback)
{
	// convert to chunk references
	TArray<TSharedRef<FChunk>> ChunksToMount;
	for (int32 ChunkId : ChunkIds)
	{
		TSharedRef<FChunk>* ChunkPtr = Chunks.Find(ChunkId);
		if (ChunkPtr != nullptr)
		{
			TSharedRef<FChunk>& ChunkRef = *ChunkPtr;
			if (ChunkRef->PakFiles.Num() > 0)
			{
				if (!ChunkRef->bIsMounted)
				{
					ChunksToMount.Add(ChunkRef);
				}
				continue;
			}
		}
		UE_LOGF(LogChunkDownloader, Warning, "Ignoring mount request for chunk %d (no mapped pak files).", ChunkId);
	}

	// make sure there are some chunks to mount (saves a frame)
	if (ChunksToMount.Num() <= 0)
	{
		// trivial success
		ExecuteNextTick(Callback, true);
		return;
	}

	// if there's no callback for some reason, avoid a bunch of boilerplate
#ifndef PVS_STUDIO // Build machine refuses to disable this warning
	if (Callback)
	{
		// loop over chunks and issue individual callback
		FMultiCallback* MultiCallback = new FMultiCallback(Callback);
		for (const TSharedRef<FChunk>& Chunk : ChunksToMount)
		{
			MountChunkInternal(*Chunk, MultiCallback->AddPending());
		}
		check(MultiCallback->GetNumPending() > 0);
	}
	else
	{
		// no need to manage callbacks
		for (const TSharedRef<FChunk>& Chunk : ChunksToMount)
		{
			MountChunkInternal(*Chunk, FCallback());
		}
	}
#endif

	// resave manifest if needed
	SaveLocalManifest(false);
	ComputeLoadingStats();
}

#undef LOCTEXT_NAMESPACE
