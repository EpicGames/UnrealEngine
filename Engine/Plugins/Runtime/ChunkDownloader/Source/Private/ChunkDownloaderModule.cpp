// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChunkDownloader.h"

#include "ChunkDownloaderLog.h"
#include "Modules/ModuleManager.h"
#include "GenericPlatform/GenericPlatformChunkInstall.h"

DEFINE_LOG_CATEGORY(LogChunkDownloader);

class FChunkDownloaderPlatformWrapper
	: public FGenericPlatformChunkInstall
{
public:
	virtual EChunkLocation::Type GetChunkLocation(uint32 ChunkID) override
	{
		// chunk 0 is special, it's always shipped with the app (by definition) so don't report it missing.
		if (ChunkID == 0)
		{
			return EChunkLocation::BestLocation;
		}

		// if the chunk downloader's not initialized, report all chunks missing
		if (!ChunkDownloader.IsValid())
		{
			return EChunkLocation::DoesNotExist;
		}

		// map the downloader's status to the chunk install interface enum
		switch (ChunkDownloader->GetChunkStatus(ChunkID))
		{
		case FChunkDownloader::EChunkStatus::Mounted:
			return EChunkLocation::BestLocation;
		case FChunkDownloader::EChunkStatus::Remote:
		case FChunkDownloader::EChunkStatus::Partial:
		case FChunkDownloader::EChunkStatus::Downloading:
		case FChunkDownloader::EChunkStatus::Cached:
			return EChunkLocation::NotAvailable;
		}
		return EChunkLocation::DoesNotExist;
	}

	virtual bool PrioritizeChunk(uint32 ChunkID, EChunkPriority::Type Priority) override
	{
		if (!ChunkDownloader.IsValid())
		{
			return false;
		}
		ChunkDownloader->MountChunk(ChunkID, FChunkDownloader::FCallback());
		return true;
	}

	virtual FDelegateHandle AddChunkInstallDelegate(FPlatformChunkInstallDelegate Delegate) override
	{
		// create if necessary
		if (!ChunkDownloader.IsValid())
		{
			ChunkDownloader = MakeShareable(new FChunkDownloader());
		}
		return ChunkDownloader->OnChunkMounted.Add(Delegate);
	}

	virtual void RemoveChunkInstallDelegate(FDelegateHandle Delegate) override
	{
		if (!ChunkDownloader.IsValid())
		{
			return;
		}
		ChunkDownloader->OnChunkMounted.Remove(Delegate);
	}

public: // trivial
	virtual EChunkInstallSpeed::Type GetInstallSpeed() override { return EChunkInstallSpeed::Fast; }
	virtual bool SetInstallSpeed(EChunkInstallSpeed::Type InstallSpeed) override { return false; }
	virtual bool DebugStartNextChunk() override { return false; }
	virtual bool GetProgressReportingTypeSupported(EChunkProgressReportingType::Type ReportType) override { return false; }
	virtual float GetChunkProgress(uint32 ChunkID, EChunkProgressReportingType::Type ReportType) override { return 0; }

public:
	FChunkDownloaderPlatformWrapper(TSharedPtr<FChunkDownloader>& ChunkDownloaderRef)
		: ChunkDownloader(ChunkDownloaderRef)
	{
	}
	virtual ~FChunkDownloaderPlatformWrapper()
	{
	}
private:
	TSharedPtr<FChunkDownloader>& ChunkDownloader;
};

/**
* Mcp Profile System Module
*/
class FChunkDownloaderModule
	: public IPlatformChunkInstallModule
{
public:
	FChunkDownloaderModule()
		: ChunkInstallWrapper(new FChunkDownloaderPlatformWrapper(ChunkDownloader))
	{
	}

	virtual IPlatformChunkInstall* GetPlatformChunkInstall() override
	{
		return ChunkInstallWrapper.Get();
	}

	// IModuleInterface interface
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
		if (ChunkDownloader.IsValid())
		{
			ChunkDownloader->Finalize();
		}
	}

	TSharedPtr<FChunkDownloader> ChunkDownloader;
	TUniquePtr<FChunkDownloaderPlatformWrapper> ChunkInstallWrapper;
};

static const FName ChunkDownloaderModuleName = "ChunkDownloader";

//static 
TSharedPtr<FChunkDownloader> FChunkDownloader::Get()
{
	FChunkDownloaderModule* Module = FModuleManager::LoadModulePtr<FChunkDownloaderModule>(ChunkDownloaderModuleName);
	if (Module != nullptr)
	{
		// may still be null
		return Module->ChunkDownloader;
	}
	return TSharedPtr<FChunkDownloader>();
}

//static 
TSharedRef<FChunkDownloader> FChunkDownloader::GetChecked()
{
	FChunkDownloaderModule& Module = FModuleManager::LoadModuleChecked<FChunkDownloaderModule>(ChunkDownloaderModuleName);
	return Module.ChunkDownloader.ToSharedRef();
}

//static 
TSharedRef<FChunkDownloader> FChunkDownloader::GetOrCreate()
{
	FChunkDownloaderModule& Module = FModuleManager::LoadModuleChecked<FChunkDownloaderModule>(ChunkDownloaderModuleName);
	if (!Module.ChunkDownloader.IsValid())
	{
		Module.ChunkDownloader = MakeShareable(new FChunkDownloader());
	}
	return Module.ChunkDownloader.ToSharedRef();
}

//static 
void FChunkDownloader::Shutdown()
{
	FChunkDownloaderModule* Module = FModuleManager::LoadModulePtr<FChunkDownloaderModule>(ChunkDownloaderModuleName);
	if (Module != nullptr)
	{
		// may still be null
		if (Module->ChunkDownloader.IsValid())
		{
			Module->ChunkDownloader->Finalize();
			Module->ChunkDownloader.Reset();
		}
	}
}

IMPLEMENT_MODULE(FChunkDownloaderModule, ChunkDownloader);

