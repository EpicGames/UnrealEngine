// Copyright Epic Games, Inc. All Rights Reserved.

#include "Download.h"

#include "ChunkDownloaderLog.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFile.h"
#include "Interfaces/IHttpResponse.h"

#define LOCTEXT_NAMESPACE "ChunkDownloader"

FDownload::FDownload(const TSharedRef<FChunkDownloader>& DownloaderIn, const TSharedRef<FChunkDownloader::FPakFile>& PakFileIn)
	: Downloader(DownloaderIn)
	, PakFile(PakFileIn)
	, TargetFile(Downloader->CacheFolder / PakFileIn->Entry.FileName)
{
	// couple of sanity checks for our flags
	check(!PakFile->bIsCached);
	check(!PakFile->bIsEmbedded);
	check(!PakFile->bIsMounted);
}

FDownload::~FDownload()
{
}

void FDownload::Start()
{
	check(!bHasCompleted);

	// check to make sure we have enough space for this download
	if (!HasDeviceSpaceRequired())
	{
		TWeakPtr<FDownload> WeakThisPtr = AsShared();
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([WeakThisPtr](float Unused) {
			TSharedPtr<FDownload> SharedThis = WeakThisPtr.Pin();
			if (SharedThis.IsValid())
			{
				SharedThis->OnCompleted(false, LOCTEXT("NotEnoughSpace", "Not enough space on device."));
			}
			return false;
		}), 1.0);
		return;
	}

	// try to download from the CDN
	StartDownload(0);
}

void FDownload::Cancel(bool bResult)
{
	check(!bHasCompleted);
	UE_LOGF(LogChunkDownloader, Warning, "Canceling download of '%ls'. result=%ls", *PakFile->Entry.FileName, bResult ? TEXT("true") : TEXT("false"));

	// cancel the platform specific file download
	if (!bIsCancelled)
	{
		bIsCancelled = true;
		CancelCallback();
	}

	// fire the completion results
	OnCompleted(bResult, FText::Format(LOCTEXT("DownloadCanceled", "Download of '%s' was canceled."), FText::FromString(PakFile->Entry.FileName)));
}

void FDownload::UpdateFileSize()
{
	IFileManager& FileManager = IFileManager::Get();
	int64 FileSizeOnDisk = FileManager.FileSize(*TargetFile);
	PakFile->SizeOnDisk = (FileSizeOnDisk > 0) ? (uint64)FileSizeOnDisk : 0;
}

bool FDownload::ValidateFile() const
{
	if (PakFile->SizeOnDisk != PakFile->Entry.FileSize)
	{
		UE_LOGF(LogChunkDownloader, Error, "Size mismatch. Expected %llu, got %llu", PakFile->Entry.FileSize, PakFile->SizeOnDisk);
		return false;
	}

	if (PakFile->Entry.FileVersion.StartsWith(TEXT("SHA1:")))
	{
		// check the sha1 hash
		if (!FChunkDownloader::CheckFileSha1Hash(TargetFile, PakFile->Entry.FileVersion))
		{
			UE_LOGF(LogChunkDownloader, Error, "Checksum mismatch. Expected %ls", *PakFile->Entry.FileVersion);
			return false;
		}
	}

	return true;
}

bool FDownload::HasDeviceSpaceRequired() const
{
	uint64 TotalDiskSpace = 0;
	uint64 TotalDiskFreeSpace = 0;
	if (FPlatformMisc::GetDiskTotalAndFreeSpace(Downloader->CacheFolder, TotalDiskSpace, TotalDiskFreeSpace))
	{
		uint64 BytesNeeded = PakFile->Entry.FileSize - PakFile->SizeOnDisk;
		if (TotalDiskFreeSpace < BytesNeeded)
		{
			// not enough space
			UE_LOGF(LogChunkDownloader, Warning, "Unable to download '%ls'. Needed %llu bytes had %llu bytes free (of %llu bytes)",
				*PakFile->Entry.FileName, PakFile->Entry.FileSize, TotalDiskFreeSpace, TotalDiskSpace);
			return false;
		}
	}
	return true;
}

void FDownload::StartDownload(int TryNumber)
{
	// only handle completion once
	check(!bHasCompleted);
	BeginTime = FDateTime::UtcNow();
	OnDownloadProgress(0);

	// download the next url
	check(Downloader->BuildBaseUrls.Num() > 0);
	FString Url = Downloader->BuildBaseUrls[TryNumber % Downloader->BuildBaseUrls.Num()] / PakFile->Entry.RelativeUrl;
	UE_LOGF(LogChunkDownloader, Log, "Downloading %ls from %ls", *PakFile->Entry.FileName, *Url);
	TWeakPtr<FDownload> WeakThisPtr = AsShared();
	CancelCallback = PlatformStreamDownload(Url, TargetFile, [WeakThisPtr](int32 BytesReceived) {
		TSharedPtr<FDownload> SharedThis = WeakThisPtr.Pin();
		if (SharedThis.IsValid() && !SharedThis->bHasCompleted)
		{
			SharedThis->OnDownloadProgress(BytesReceived);
		}
	}, [WeakThisPtr, TryNumber, Url](int32 HttpStatus) {
		TSharedPtr<FDownload> SharedThis = WeakThisPtr.Pin();
		if (SharedThis.IsValid() && !SharedThis->bHasCompleted)
		{
			SharedThis->OnDownloadComplete(Url, TryNumber, HttpStatus);
		}
	});
}

void FDownload::OnDownloadProgress(int32 BytesReceived)
{
	Downloader->LoadingModeStats.BytesDownloaded -= LastBytesReceived;
	LastBytesReceived = BytesReceived;
	Downloader->LoadingModeStats.BytesDownloaded += LastBytesReceived;
}

void FDownload::OnDownloadComplete(const FString& Url, int TryNumber, int32 HttpStatus)
{
	// only handle completion once
	check(!bHasCompleted);

	// update file size on disk
	UpdateFileSize();

	// report analytics
	if (Downloader->OnDownloadAnalytics)
	{
		Downloader->OnDownloadAnalytics(PakFile->Entry.FileName, Url, PakFile->SizeOnDisk, FDateTime::UtcNow() - BeginTime, HttpStatus);
	}

	// handle success
	if (EHttpResponseCodes::IsOk(HttpStatus))
	{
		// make sure the file is complete
		if (ValidateFile())
		{
			PakFile->bIsCached = true;
			OnCompleted(true, FText());
			return;
		}

		// if we fail validation, delete the file and start over
		UE_LOGF(LogChunkDownloader, Error, "%ls from %ls failed validation", *TargetFile, *Url);
		IPlatformFile::GetPlatformPhysical().DeleteFile(*TargetFile);
	}

	// check again to make sure we have enough space for this download
	if (!HasDeviceSpaceRequired())
	{
		OnCompleted(false, LOCTEXT("NotEnoughSpace", "Not enough space on device."));
		return;
	}

	// compute delay before re-starting download
	float SecondsToDelay = (TryNumber + 1) * 5.0f;
	if (SecondsToDelay > 60)
	{
		SecondsToDelay = 60;
	}

	// set a ticker to delay
	UE_LOGF(LogChunkDownloader, Log, "Will re-attempt to download %ls in %f seconds", *PakFile->Entry.FileName, SecondsToDelay);
	TWeakPtr<FDownload> WeakThisPtr = AsShared();
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([WeakThisPtr, TryNumber](float Unused) {
		TSharedPtr<FDownload> SharedThis = WeakThisPtr.Pin();
		if (SharedThis.IsValid() && !SharedThis->bHasCompleted)
		{
			SharedThis->StartDownload(TryNumber + 1);
		}
		return false;
	}), SecondsToDelay);
}

void FDownload::OnCompleted(bool bSuccess, const FText& ErrorText)
{
	// make sure we don't complete more than once
	check(!bHasCompleted);
	bHasCompleted = true;

	// increment files downloaded
	OnDownloadProgress(bSuccess ? PakFile->SizeOnDisk : 0);
	++Downloader->LoadingModeStats.FilesDownloaded;
	if (!bSuccess && !ErrorText.IsEmpty())
	{
		Downloader->LoadingModeStats.LastError = ErrorText;
	}

	// queue up callbacks
	for (const auto& Callback : PakFile->PostDownloadCallbacks)
	{
		Downloader->ExecuteNextTick(Callback, bSuccess);
	}
	PakFile->PostDownloadCallbacks.Empty();

	// remove from download requests
	if (ensure(Downloader->DownloadRequests.RemoveSingle(PakFile) > 0))
	{
		Downloader->IssueDownloads();
	}

	// unhook from pak file (this may delete us)
	if (PakFile->Download.Get() == this)
	{
		PakFile->Download.Reset();
	}
}

#undef LOCTEXT_NAMESPACE
