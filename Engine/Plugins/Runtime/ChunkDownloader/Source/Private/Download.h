// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChunkDownloader.h"
#include "PlatformStreamDownload.h"

class FDownload : public TSharedFromThis<FDownload>
{
public:
	FDownload(const TSharedRef<FChunkDownloader>& DownloaderIn, const TSharedRef<FChunkDownloader::FPakFile>& PakFileIn);
	virtual ~FDownload();

	inline bool HasCompleted() const { return bHasCompleted; }
	inline int32 GetProgress() const { return LastBytesReceived; }

	void Start();
	void Cancel(bool bResult);

public:
	const TSharedRef<FChunkDownloader> Downloader;
	const TSharedRef<FChunkDownloader::FPakFile> PakFile;
	const FString TargetFile;

protected:
	void UpdateFileSize();
	bool ValidateFile() const;
	bool HasDeviceSpaceRequired() const;
	void StartDownload(int TryNumber);
	void OnDownloadProgress(int32 BytesReceived);
	void OnDownloadComplete(const FString& Url, int TryNumber, int32 HttpStatus);
	void OnCompleted(bool bSuccess, const FText& ErrorText);

private:
	bool bIsCancelled = false;
	FDownloadCancel CancelCallback;
	bool bHasCompleted = false;
	FDateTime BeginTime;
	int32 LastBytesReceived = 0;
};
