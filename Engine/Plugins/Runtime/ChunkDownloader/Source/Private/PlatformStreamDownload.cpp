// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlatformStreamDownload.h"
#include "ChunkDownloaderLog.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFile.h"
#include "HttpModule.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"
#include "Modules/ModuleManager.h"

//////////////////////////////////////////////////////////////////////////////////
#if 0 && PLATFORM_ANDROID

// Android
// https://developer.android.com/reference/android/app/DownloadManager.html
#error "TODO: android"
FDownloadCancel PlatformStreamDownload(const FString& Url, const FString& TargetFile, const FDownloadProgress& Progress, const FDownloadComplete& Callback)
{
	// TODO: write me
	Callback(0);
	// not cancelable
	return []() {};
}

//////////////////////////////////////////////////////////////////////////////////
#elif 0 && PLATFORM_IOS

// iOS
// https://developer.apple.com/library/content/documentation/iPhone/Conceptual/iPhoneOSProgrammingGuide/BackgroundExecution/BackgroundExecution.html
#error "TODO: ios"
FDownloadCancel PlatformStreamDownload(const FString& Url, const FString& TargetFile, const FDownloadProgress& Progress, const FDownloadComplete& Callback)
{
	// TODO: write me
	Callback(0);
	// not cancelable
	return []() {};
}

//////////////////////////////////////////////////////////////////////////////////
#else

// NOTE: this implementation does not stream the file, it loads the whole thing into memory
// then saves it (not optimal). It does attempt to resume interrupted downloads (for use in testing), but since it doesn't do partial writes, those probably won't occur in the wild.
FDownloadCancel PlatformStreamDownload(const FString& Url, const FString& TargetFile, const FDownloadProgress& Progress, const FDownloadComplete& Callback)
{
	// how much of the file do we currently have on disk (if any)
	IFileManager& FileManager = IFileManager::Get();
	int64 FileSizeOnDisk = FileManager.FileSize(*TargetFile);
	uint64 SizeOnDisk = (FileSizeOnDisk > 0) ? (uint64)FileSizeOnDisk : 0;

	// do a range request for the part we're missing
	FHttpModule& HttpModule = FModuleManager::LoadModuleChecked<FHttpModule>("HTTP");
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> Request = HttpModule.Get().CreateRequest();
	Request->SetURL(Url);
	Request->SetVerb(TEXT("GET"));
	if (SizeOnDisk > 0)
	{
		// try to request a specific range
		Request->SetHeader(TEXT("Range"), FString::Printf(TEXT("bytes=%llu-"), SizeOnDisk));
	}

	// bind the progress delegate
	if (Progress)
	{
		Request->OnRequestProgress64().BindLambda([Progress](FHttpRequestPtr HttpRequest, uint64 BytesSent, uint64 BytesReceived) {
			Progress(BytesReceived);
		});
	}
	
	// bind a completion delegate
	Request->OnProcessRequestComplete().BindLambda([Callback, TargetFile, SizeOnDisk](FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSuccess) {
		// check response
		int32 HttpStatus = 0;
		if (HttpResponse.IsValid())
		{
			HttpStatus = HttpResponse->GetResponseCode();
			bool bHeadersOk = EHttpResponseCodes::IsOk(HttpStatus);
			const bool bIsPartialContent = (HttpStatus == 206);
			if (bIsPartialContent)
			{
				static const FString ContentRangeHeader = TEXT("Content-Range");
				// if we got partial content, make sure the Content-Range header is what we expect
				FString ExpectedHeaderPrefix = FString::Printf(TEXT("bytes %llu-"), SizeOnDisk);
				FString HeaderValue = HttpResponse->GetHeader(ContentRangeHeader);
				if (!HeaderValue.StartsWith(ExpectedHeaderPrefix))
				{
					UE_LOGF(LogChunkDownloader, Error, "Content-Range for %ls was '%ls' but expected '%ls' prefix", *HttpRequest->GetURL(), *HeaderValue, *ExpectedHeaderPrefix);
					bHeadersOk = false;
				}
			}

			// see if the headers are alright
			if (bHeadersOk)
			{
				// open the file for writing
				IFileHandle* ManifestFile = IPlatformFile::GetPlatformPhysical().OpenWrite(*TargetFile, SizeOnDisk > 0 && bIsPartialContent);
				if (ManifestFile != nullptr)
				{
					// write to the file
					const TArray<uint8>& Content = HttpResponse->GetContent();
					bSuccess = ManifestFile->Write(&Content[0], Content.Num());
					// close the file
					delete ManifestFile;

					// handle failure
					if (!bSuccess)
					{
						UE_LOGF(LogChunkDownloader, Error, "Write error writing to %ls", *TargetFile);

						// delete the file (space issue?)
						IPlatformFile::GetPlatformPhysical().DeleteFile(*TargetFile);
					}
				}
				else
				{
					UE_LOGF(LogChunkDownloader, Error, "Unable to save file to %ls", *TargetFile);

					// delete the file (space issue?)
					if (SizeOnDisk > 0)
					{
						IPlatformFile::GetPlatformPhysical().DeleteFile(*TargetFile);
					}
				}
			}
			else
			{
				UE_LOGF(LogChunkDownloader, Error, "HTTP %d returned from '%ls'", HttpStatus, *HttpRequest->GetURL());

				// if the server responded with anything not ok (and not a server error), then delete the file for next time
				if (HttpStatus < 500 && SizeOnDisk > 0)
				{
					IPlatformFile::GetPlatformPhysical().DeleteFile(*TargetFile);
				}
			}
		}
		else
		{
			UE_LOGF(LogChunkDownloader, Error, "HTTP connection issue downloading '%ls'", *HttpRequest->GetURL());
		}

		// invoke the callback
		if (Callback)
		{
			Callback(HttpStatus);
		}
	});
	Request->ProcessRequest();
	return [Request]() {
		Request->CancelRequest();
	};
}
#endif
