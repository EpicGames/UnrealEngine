// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

class FString;
template <typename FuncType> class TFunction;

typedef TFunction<void(int32 HttpStatus)> FDownloadComplete;
typedef TFunction<void(uint64 BytesReceived)> FDownloadProgress;
typedef TFunction<void(void)> FDownloadCancel;

extern FDownloadCancel PlatformStreamDownload(const FString& Url, const FString& TargetFile, const FDownloadProgress& Progress, const FDownloadComplete& Callback);
