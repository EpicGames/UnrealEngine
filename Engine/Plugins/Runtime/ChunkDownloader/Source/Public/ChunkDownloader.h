// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Internationalization/Text.h"
#include "Misc/DateTime.h"

#define UE_API CHUNKDOWNLOADER_API

template<typename TTask> class FAsyncTask;
class IHttpRequest;
class FDownload;

struct FPakFileEntry
{
	// unique name of the pak file (not path, i.e. no folder)
	FString FileName;
	
	// final size of the file in bytes
	uint64 FileSize = 0;

	// unique ID representing a particular version of this pak file
	// when it is used for validation (not done on golden path, but can be requested) this is assumed 
	// to be a SHA1 hash if it begins with "SHA1:" otherwise it's considered just a unique ID.
	FString FileVersion;

	// chunk ID this pak file is assigned to
	int32 ChunkId = -1;

	// URL for this pak file (relative to CDN root, includes build-specific folder)
	FString RelativeUrl;
};

DECLARE_MULTICAST_DELEGATE_TwoParams(FPlatformChunkInstallMultiDelegate, uint32, bool);

class FChunkDownloader : public TSharedFromThis<FChunkDownloader>
{
public:
	UE_API ~FChunkDownloader();
	typedef TFunction<void(bool bSuccess)> FCallback;

	// static getters
	static UE_API TSharedPtr<FChunkDownloader> Get();
	static UE_API TSharedRef<FChunkDownloader> GetChecked();
	static UE_API TSharedRef<FChunkDownloader> GetOrCreate();
	static UE_API void Shutdown();

	// initialize the download manager (populates the list of cached pak files from disk). Call only once.
	UE_API void Initialize(const FString& PlatformName, int32 TargetDownloadsInFlight);

	// unmount all chunks and cancel any downloads in progress (preserving partial downloads). 
	// Call only once, don't reuse this object, make a new one.
	UE_API void Finalize();

	// try to load a cached build ID from disk (good to do before updating build so it can possibly no-op)
	UE_API bool LoadCachedBuild(const FString& DeploymentName);

	// set the the content build id
	// if the content build id has changed, we pull the new BuildManifest from CDN and load it.
	// the client should compare ContentBuildId with its current embedded build id to determine if this content is 
	// even compatible BEFORE calling this function. e.g. ContentBuildId="v1.4.22-r23928293" we might consider BUILD_VERSION="1.4.1" 
	// compatible but BUILD_VERSION="1.3.223" incompatible (needing an update)
	UE_API void UpdateBuild(const FString& DeploymentName, const FString& ContentBuildId, const FCallback& Callback);

	// get the current content build ID
	inline const FString& GetContentBuildId() const { return ContentBuildId; }
	// get the most recent deployment name
	inline const FString& GetDeploymentName() const { return LastDeploymentName; }

	enum class EChunkStatus
	{
		Mounted, // chunk is cached locally and mounted in RAM
		Cached, // chunk is fully cached locally but not mounted
		Downloading, // chunk is partially cached locally, not mounted, download in progress
		Partial, // chunk is partially cached locally, not mounted, download NOT in progress
		Remote, // no local caching has started
		Unknown, // no paks are included in this chunk, can consider it either an error or fully mounted depending
	};

	static UE_API void DumpLoadedChunks();

	// chunk status as logable string
	static UE_API const TCHAR* ChunkStatusToString(EChunkStatus Status);

	// get the current status of the specified chunk
	UE_API EChunkStatus GetChunkStatus(int32 ChunkId) const;

	// return a list of all chunk IDs in the current manifest
	UE_API void GetAllChunkIds(TArray<int32>& OutChunkIds) const;

	// Download and mount all chunks then fire the callback (convenience wrapper managing multiple MountChunk calls)
	UE_API void MountChunks(const TArray<int32>& ChunkIds, const FCallback& Callback);

	// download all pak files, then asynchronously mount them in order (in order among themselves, async with game thread). 
	UE_API void MountChunk(int32 ChunkId, const FCallback& Callback);

	// Download (Cache) all pak files in these chunks then fire the callback (convenience wrapper managing multiple DownloadChunk calls)
	UE_API void DownloadChunks(const TArray<int32>& ChunkIds, const FCallback& Callback, int32 Priority = 0);

	// download all pak files in the chunk, but don't mount. Callback is fired when all paks have finished caching 
	// (whether success or failure). Downloads will retry forever, but might fail due to space issues.
	UE_API void DownloadChunk(int32 ChunkId, const FCallback& Callback, int32 Priority = 0);

	// flush any cached files (on disk) that are not currently being downloaded to or mounting (does not unmount the corresponding pak files).
	// this will include full and partial downloads, but not active downloads.
	UE_API int FlushCache();

	// validate all fully cached files (blocking) by attempting to read them and check their Version hash.
	// this automatically deletes any files that don't match. Returns the number of files deleted.
	// in this case best to return to a simple update map and reinitialize ChunkDownloader (or restart).
	UE_API int ValidateCache();

	// Snapshot stats and enter into loading screen mode (pauses all background downloads). Fires callback when all non-background 
	// downloads have completed. If no downloads/mounts are currently queued by the end of the frame, callback will fire next frame.
	UE_API void BeginLoadingMode(const FCallback& Callback);

	struct FStats
	{
		// number of pak files downloaded
		int FilesDownloaded = 0;
		int TotalFilesToDownload = 0;

		// number of bytes downloaded
		uint64 BytesDownloaded = 0;
		uint64 TotalBytesToDownload = 0;
		
		// number of chunks mounted (chunk is an ordered array of paks)
		int ChunksMounted = 0;
		int TotalChunksToMount = 0;
		
		// UTC time that loading began (for rate estimates)
		FDateTime LoadingStartTime = FDateTime::MinValue();
		FText LastError;
	};

	// get the current loading stats (generally only useful if you're in loading mode see BeginLoadingMode)
	inline const FStats& GetLoadingStats() const { return LoadingModeStats; }

	// Called whenever a chunk mounts (success or failure). ONLY USE THIS IF YOU WANT TO PASSIVELY LISTEN FOR MOUNTS (otherwise use the proper request callback on MountChunk)
	FPlatformChunkInstallMultiDelegate OnChunkMounted;

	// called each time a download attempt finishes (success or failure). ONLY USE THIS IF YOU WANT TO PASSIVELY LISTEN. Downloads retry until successful.
	TFunction<void(const FString& FileName, const FString& Url, uint64 SizeBytes, const FTimespan& DownloadTime, int32 HttpStatus)> OnDownloadAnalytics;

	// get current number of download requests, so we know whether download is in progress. Downlading Requests will be removed from this array in it's FDownload::OnCompleted callback.
	inline int32 GetNumDownloadRequests() const { return DownloadRequests.Num(); }

protected:
	friend class FChunkDownloaderModule;
	friend class FChunkDownloaderPlatformWrapper;
	friend class FDownload;
	UE_API FChunkDownloader();

	static UE_API bool CheckFileSha1Hash(const FString& FullPathOnDisk, const FString& Sha1HashStr);

private:
	struct FChunk;
	struct FPakFile;

	UE_API void SetContentBuildId(const FString& DeploymentName, const FString& NewContentBuildId);

	UE_API void LoadManifest(const TArray<FPakFileEntry>& PakFiles);
	UE_API void TryLoadBuildManifest(int TryNumber);
	UE_API void TryDownloadBuildManifest(int TryNumber);

	UE_API void WaitForMounts();
	UE_API void SaveLocalManifest(bool bForce);

	UE_API bool UpdateLoadingMode();
	UE_API void ComputeLoadingStats();

	UE_API void UnmountPakFile(const TSharedRef<FPakFile>& PakFile);
	UE_API void CancelDownload(const TSharedRef<FPakFile>& PakFile, bool bResult);
	UE_API void DownloadPakFileInternal(const TSharedRef<FPakFile>& PakFile, const FCallback& Callback, int32 Priority);

	UE_API void MountChunkInternal(FChunk& Chunk, const FCallback& Callback);
	UE_API void DownloadChunkInternal(const FChunk& Chunk, const FCallback& Callback, int32 Priority);
	UE_API void CompleteMountTask(FChunk& Chunk);

	UE_API bool UpdateMountTasks(float dts);
	UE_API void ExecuteNextTick(const FCallback& Callback, bool bSuccess);

	UE_API void IssueDownloads();

private:
	class FMultiCallback;

	// entry per pak file
	struct FPakFile
	{
		FPakFileEntry Entry;
		bool bIsCached = false;
		bool bIsMounted = false;

		bool bIsEmbedded = false;
		uint64 SizeOnDisk = 0; // grows as the file is downloaded. See Entry.FileSize for the target size

		// async download
		int32 Priority = 0;
		TSharedPtr<FDownload> Download;
		TArray<FChunkDownloader::FCallback> PostDownloadCallbacks;
	};

	// represents an async mount
	class FPakMountWork;
	typedef FAsyncTask<FPakMountWork> FMountTask;

	// entry per chunk
	struct FChunk
	{
		int32 ChunkId = -1;
		bool bIsMounted = false;

		TArray<TSharedRef<FPakFile>> PakFiles;

		inline bool IsCached() const
		{
			for (const auto& PakFile : PakFiles)
			{
				if (!PakFile->bIsCached)
				{
					return false;
				}
			}
			return true;
		}

		// async mount
		FMountTask* MountTask = nullptr;
	};

private:
	// cumulative stats for loading screen mode
	FStats LoadingModeStats;
	TArray<FCallback> PostLoadCallbacks;
	int32 LoadingCompleteLatch = 0;

	FCallback UpdateBuildCallback;

	// platform name (determines the manifest)
	FString PlatformName;

	// folders to save pak files into on disk
	FString CacheFolder;

	// content folder where we can find some chunks shipped with the build
	FString EmbeddedFolder;

	// build specific ID and URL paths
	FString LastDeploymentName;
	FString ContentBuildId;
	TArray<FString> BuildBaseUrls;

	// chunk id to chunk record
	TMap<int32,TSharedRef<FChunk>> Chunks;

	// pak file name to pak file record
	TMap<FString,TSharedRef<FPakFile>> PakFiles;

	// pak files embedded in the build (immutable, compressed)
	TMap<FString,FPakFileEntry> EmbeddedPaks;

	// do we need to save the manifest (done whenever new downloads have started)
	bool bNeedsManifestSave = false;

	// handle for the per-frame mount ticker in the main thread
	FTSTicker::FDelegateHandle MountTicker;

	// manifest download request
	TSharedPtr<IHttpRequest, ESPMode::ThreadSafe> ManifestRequest;

	// maximum number of downloads to allow concurrently
	int32 TargetDownloadsInFlight = 1;

	// list of pak files that have been requested
	TArray<TSharedRef<FPakFile>> DownloadRequests;
};

#undef UE_API
